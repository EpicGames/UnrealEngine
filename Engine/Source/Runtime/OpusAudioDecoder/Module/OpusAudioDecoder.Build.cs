// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OpusAudioDecoder : ModuleRules
{
    public OpusAudioDecoder(ReadOnlyTargetRules Target) : base(Target)
    {
        ShortName = "OpusAudioDecoder";

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine"
            }
        );
        
        AddEngineThirdPartyPrivateStaticDependencies(Target,
	        "UEOgg",
	        "libOpus"
        );        

	    PublicDefinitions.Add("WITH_OPUS=1");
    }
}
