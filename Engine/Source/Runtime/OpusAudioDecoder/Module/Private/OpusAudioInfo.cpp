// Copyright Epic Games, Inc. All Rights Reserved.


#include "Decoders/OpusAudioInfo.h"
#include "Interfaces/IAudioFormat.h"
#include <opus_defines.h>
#include <opus_types.h>

THIRD_PARTY_INCLUDES_START
#include "opus_multistream.h"
THIRD_PARTY_INCLUDES_END

#define USE_UE_MEM_ALLOC 1

DEFINE_LOG_CATEGORY_STATIC(LogOpusAudioDecoder, Log, All);

///////////////////////////////////////////////////////////////////////////////////////
// Followed pattern used in opus_multistream_encoder.c - this will allow us to setup //
// a multistream decoder without having to save extra information for every asset.   //
///////////////////////////////////////////////////////////////////////////////////////
struct UnrealChannelLayout{
	int32 NumStreams;
	int32 NumCoupledStreams;
	uint8 Mapping[8];
};

/* Index is NumChannels-1*/
static const UnrealChannelLayout UnrealMappings[8] = {
	{ 1, 0, { 0 } },                      /* 1: mono */
	{ 1, 1, { 0, 1 } },                   /* 2: stereo */
	{ 2, 1, { 0, 1, 2 } },                /* 3: 1-d surround */
	{ 2, 2, { 0, 1, 2, 3 } },             /* 4: quadraphonic surround */
	{ 3, 2, { 0, 1, 4, 2, 3 } },          /* 5: 5-channel surround */
	{ 4, 2, { 0, 1, 4, 5, 2, 3 } },       /* 6: 5.1 surround */
	{ 4, 3, { 0, 1, 4, 6, 2, 3, 5 } },    /* 7: 6.1 surround */
	{ 5, 3, { 0, 1, 6, 7, 2, 3, 4, 5 } }, /* 8: 7.1 surround */
};

/*------------------------------------------------------------------------------------
FOpusDecoderWrapper
------------------------------------------------------------------------------------*/
struct FOpusDecoderWrapper
{
	FOpusDecoderWrapper(uint32 SampleRate, uint8 NumChannels)
	{
		check(NumChannels <= 8);
		const UnrealChannelLayout& Layout = UnrealMappings[NumChannels-1];
	#if USE_UE_MEM_ALLOC
		int32 DecSize = opus_multistream_decoder_get_size(Layout.NumStreams, Layout.NumCoupledStreams);
		Decoder = (OpusMSDecoder*)FMemory::Malloc(DecSize);
		DecError = opus_multistream_decoder_init(Decoder, SampleRate, NumChannels, Layout.NumStreams, Layout.NumCoupledStreams, Layout.Mapping);
	#else
		Decoder = opus_multistream_decoder_create(SampleRate, NumChannels, Layout.NumStreams, Layout.NumCoupledStreams, Layout.Mapping, &DecError);
	#endif
	}

	~FOpusDecoderWrapper()
	{
	#if USE_UE_MEM_ALLOC
		FMemory::Free(Decoder);
	#else
		opus_multistream_decoder_destroy(Decoder);
	#endif
	}

	int32 Decode(const uint8* FrameData, uint16 FrameSize, int16* OutPCMData, int32 SampleSize)
	{
		return opus_multistream_decode(Decoder, FrameData, FrameSize, OutPCMData, SampleSize, 0);
	}

	bool WasInitialisedSuccessfully() const
	{
		return DecError == OPUS_OK;
	}

private:
	OpusMSDecoder* Decoder;
	int32 DecError;
};

/*------------------------------------------------------------------------------------
FOpusAudioInfo.
------------------------------------------------------------------------------------*/
FOpusAudioInfo::FOpusAudioInfo()
	: OpusDecoderWrapper(nullptr)
{
}

FOpusAudioInfo::~FOpusAudioInfo()
{
	if (OpusDecoderWrapper != nullptr)
	{
		delete OpusDecoderWrapper;
		OpusDecoderWrapper = nullptr;
	}
}


bool FOpusAudioInfo::ParseHeader(FHeader& OutHeader, uint32& OutNumRead, const uint8* InSrcBufferData, uint32 InSrcBufferDataSize)
{
	OutNumRead = 0;

	if ((int32)InSrcBufferDataSize < FHeader::HeaderSize())
	{
		return false;
	}

	auto Read = [&InSrcBufferData, &OutNumRead](void* To, int32 NumBytes) -> void
	{
		FMemory::Memcpy(To, InSrcBufferData, NumBytes);
		InSrcBufferData += NumBytes;
		OutNumRead += NumBytes;
	};

	Read(OutHeader.Identifier, 8);
	if (FMemory::Memcmp(OutHeader.Identifier, FHeader::OPUS_ID, 8))
	{
		return false;
	}
	Read(&OutHeader.Version, sizeof(uint8));
	Read(&OutHeader.NumChannels, sizeof(uint8));
	Read(&OutHeader.SampleRate, sizeof(uint32));
	Read(&OutHeader.EncodedSampleRate, sizeof(uint32));
	Read(&OutHeader.ActiveSampleCount, sizeof(uint64));
	Read(&OutHeader.NumEncodedFrames, sizeof(uint32));
	Read(&OutHeader.NumPreSkipSamples, sizeof(int32));
	Read(&OutHeader.NumSilentSamplesAtBeginning, sizeof(int32));
	Read(&OutHeader.NumSilentSamplesAtEnd, sizeof(int32));
	return true;
}

bool FOpusAudioInfo::ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, struct FSoundQualityInfo* QualityInfo)
{
	SrcBufferData = InSrcBufferData;
	SrcBufferDataSize = InSrcBufferDataSize;
	SrcBufferOffset = 0;
	CurrentSampleCount = 0;

	Header.Reset();
	if (!ParseHeader(Header, SrcBufferOffset, InSrcBufferData, InSrcBufferDataSize))
	{
		return false;
	}

	// Store the offset to where the audio data begins
	AudioDataOffset = SrcBufferOffset;
	// Sample counts in the Opus API are always per-channel so we multiply by NumChannels here
	TrueSampleCount = (uint32)Header.ActiveSampleCount * Header.NumChannels;
	NumChannels = Header.NumChannels;

	// Write out the the header info
	if (QualityInfo)
	{
		QualityInfo->SampleRate = Header.SampleRate;
		QualityInfo->NumChannels = Header.NumChannels;
		QualityInfo->SampleDataSize = (uint32)Header.ActiveSampleCount * QualityInfo->NumChannels * sizeof(int16);
		QualityInfo->Duration = (float)((double)Header.ActiveSampleCount / QualityInfo->SampleRate);
	}

	return true;
}

bool FOpusAudioInfo::CreateDecoder()
{
	check(OpusDecoderWrapper == nullptr);
	OpusDecoderWrapper = new FOpusDecoderWrapper(Header.EncodedSampleRate, NumChannels);
	if (!OpusDecoderWrapper->WasInitialisedSuccessfully())
	{
		delete OpusDecoderWrapper;
		OpusDecoderWrapper = nullptr;
		return false;
	}

	NumRemainingSamplesToSkip = Header.NumSilentSamplesAtBeginning + Header.NumPreSkipSamples;
	PreviousDecodedUnusedSamples.Empty();
	return true;
}

int32 FOpusAudioInfo::GetFrameSize()
{
	// Opus format has variable frame size at the head of each frame...
	// We have to assume that the SrcBufferOffset is at the correct location for the read
	uint16 FrameSize = 0;
	Read(&FrameSize, sizeof(uint16));
	return (int32)FrameSize;
}

uint32 FOpusAudioInfo::GetMaxFrameSizeSamples() const
{
	// The encoder is using 20ms frame sizes.
	const int32 OPUS_MAX_FRAME_SIZE_MS = 20;
	// There can be at most 2 frames in one packet, so multiply by 2.
	return Header.EncodedSampleRate * OPUS_MAX_FRAME_SIZE_MS * 2 / 1000;
}

FDecodeResult FOpusAudioInfo::Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize)
{
	FDecodeResult Result;

	if (OpusDecoderWrapper)
	{
		const int32 kFrameBytes = NumChannels * sizeof(int16);
		int32 OutputSizeToGo = OutputPCMDataSize;
		int32 CompressedSizeToGo = CompressedDataSize;
		const uint8* InputDataPtr = CompressedData;
		uint8* OutputDataPtr = OutPCMData;

		Result.NumAudioFramesProduced = 0;
		while(OutputSizeToGo > 0)
		{
			// Get anything that is left over from the previous call.
			if (PreviousDecodedUnusedSamples.Num())
			{
				check(NumRemainingSamplesToSkip == 0);
				int32 MaxToCopyOut = OutputSizeToGo >= PreviousDecodedUnusedSamples.Num() ? PreviousDecodedUnusedSamples.Num() : OutputSizeToGo;
				FMemory::Memcpy(OutputDataPtr, PreviousDecodedUnusedSamples.GetData(), MaxToCopyOut);
				PreviousDecodedUnusedSamples.RemoveAt(0, MaxToCopyOut);
				OutputSizeToGo -= MaxToCopyOut;
				OutputDataPtr += MaxToCopyOut;
				// If there is still something left over then we are done here.
				Result.NumAudioFramesProduced += MaxToCopyOut / kFrameBytes;
				if (OutputSizeToGo == 0 || PreviousDecodedUnusedSamples.Num())
				{
					Result.NumCompressedBytesConsumed = InputDataPtr - CompressedData;
					Result.NumPcmBytesProduced = Result.NumAudioFramesProduced * kFrameBytes;
					return Result;
				}
			}
			// Something left to decompress?
			if (CompressedSizeToGo <= 0)
			{
				break;
			}

			// Decode the next chunk.
			const int32 AvailableSampleOutputSize = OutputSizeToGo / kFrameBytes;

			int32 ChunkToDecodeSize = 0;
			int32 ActualChunkSize = 0;
			// Is this streaming?
			const uint8* ChunkToDecodePtr = InputDataPtr;
			if (!bIsStreaming)
			{
				// When not streaming each input data chunk is prepended with the size of the chunk.
				ChunkToDecodeSize = (int32)InputDataPtr[0] + ((int32)InputDataPtr[1] << 8);
				ActualChunkSize = ChunkToDecodeSize + 2;
				ChunkToDecodePtr += 2;
			}
			else
			{
				// When streaming the chunk size is still prepended to the chunk, but is skipped over
				// by the caller and provided as an argument.
				ChunkToDecodeSize = CompressedDataSize;
				ActualChunkSize = ChunkToDecodeSize;
			}


			int32 NumDecodedFrames = OpusDecoderWrapper->Decode(ChunkToDecodePtr, ChunkToDecodeSize, (int16*)OutputDataPtr, AvailableSampleOutputSize);
			if (NumDecodedFrames >= 0)
			{
				CompressedSizeToGo -= ActualChunkSize;
				InputDataPtr += ActualChunkSize;
				if (NumRemainingSamplesToSkip)
				{
					if (NumRemainingSamplesToSkip >= NumDecodedFrames)
					{
						NumRemainingSamplesToSkip -= NumDecodedFrames;
						NumDecodedFrames = 0;
					}
					else
					{
						uint8* FirstUsable = OutputDataPtr + NumRemainingSamplesToSkip * kFrameBytes;
						int32 UsableSize = (NumDecodedFrames - NumRemainingSamplesToSkip) * kFrameBytes;
						FMemory::Memmove(OutputDataPtr, FirstUsable, UsableSize);
						NumDecodedFrames -= NumRemainingSamplesToSkip;
						NumRemainingSamplesToSkip = 0;
					}
				}
				Result.NumAudioFramesProduced += NumDecodedFrames;
				OutputSizeToGo -= NumDecodedFrames * kFrameBytes;
				OutputDataPtr += NumDecodedFrames * kFrameBytes;
			}
			// Was the remaining output buffer too small?
			else if (NumDecodedFrames == OPUS_BUFFER_TOO_SMALL)
			{
				int32 NumSampleSpaceNeeded = GetMaxFrameSizeSamples();
				int32 NumPrevBefore = PreviousDecodedUnusedSamples.Num();
				PreviousDecodedUnusedSamples.AddUninitialized(NumSampleSpaceNeeded * kFrameBytes);
				uint8* TempPtr = PreviousDecodedUnusedSamples.GetData() + NumPrevBefore;
				NumDecodedFrames = OpusDecoderWrapper->Decode(ChunkToDecodePtr, ChunkToDecodeSize, (int16*)TempPtr, NumSampleSpaceNeeded);
				if (NumDecodedFrames >= 0)
				{
					int32 NumPrevNow = NumDecodedFrames * kFrameBytes;
					PreviousDecodedUnusedSamples.SetNum(NumPrevNow);
					CompressedSizeToGo -= ActualChunkSize;
					InputDataPtr += ActualChunkSize;
					if (NumRemainingSamplesToSkip)
					{
						const int32 NumBytesToSkip = NumRemainingSamplesToSkip * kFrameBytes;
						if (NumBytesToSkip >= NumPrevNow)
						{
							PreviousDecodedUnusedSamples.SetNum(0, EAllowShrinking::No);
							NumRemainingSamplesToSkip -= NumPrevNow / kFrameBytes;
						}
						else
						{
							PreviousDecodedUnusedSamples.RemoveAt(0, NumBytesToSkip);
							NumRemainingSamplesToSkip = 0;
						}
					}
					continue;
				}
				else
				{
					UE_LOGF(LogOpusAudioDecoder, Error, "opus_multistream_decode() returned %d while decoding into temporary buffer", NumDecodedFrames);
					return FDecodeResult();
				}
			}
			else
			{
				// A decode error of sorts.
				UE_LOGF(LogOpusAudioDecoder, Error, "opus_multistream_decode() returned %d", NumDecodedFrames);
				return FDecodeResult();
			}
		}
		Result.NumCompressedBytesConsumed = InputDataPtr - CompressedData;
		Result.NumPcmBytesProduced = Result.NumAudioFramesProduced * kFrameBytes;
	}
	return Result;
}

void FOpusAudioInfo::PrepareToLoop()
{
	IStreamedCompressedInfo::PrepareToLoop();
	NumRemainingSamplesToSkip = Header.NumSilentSamplesAtBeginning + Header.NumPreSkipSamples;
	PreviousDecodedUnusedSamples.Empty();
}

void FOpusAudioInfo::SeekToTime(const float InSeekTime)
{
	if (GetStreamingSoundWaveData().IsValid())
	{
		IStreamedCompressedInfo::SeekToTime(InSeekTime);
	}
	else if (SrcBufferData && SrcBufferDataSize)
	{
		uint32 SeekFrameNum = 0;
		if (InSeekTime > 0.0f)
		{
			SeekFrameNum = (uint32)(InSeekTime * Header.SampleRate);
		}

		SeekToFrame(SeekFrameNum);
	}
}

void FOpusAudioInfo::SeekToFrame(const uint32 InSeekFrame)
{
	int32 NumSamplesToSkipInChunk = 0;
	if (GetStreamingSoundWaveData().IsValid())
	{
		IStreamedCompressedInfo::SeekToFrame(InSeekFrame);
	}
	else if (SrcBufferData && SrcBufferDataSize)
	{
		uint32 SeekSampleNum = InSeekFrame * NumChannels;
		const uint8* ChunkPtr = SrcBufferData + AudioDataOffset;
		const uint8* const EndPtr = SrcBufferData + SrcBufferDataSize;
		uint32 CurrentChunkSampleNum = 0;
		while (ChunkPtr < EndPtr)
		{
			uint32 ChunkSize = (uint32)ChunkPtr[0] + ((uint32)ChunkPtr[1] << 8);
			int32 ExpectedFrames = opus_packet_get_nb_frames(ChunkPtr + 2, ChunkSize);
			int32 ExpectedFrameSize = opus_packet_get_samples_per_frame(ChunkPtr + 2, Header.EncodedSampleRate);
			int32 NumExpectedTotal = ExpectedFrames * ExpectedFrameSize * NumChannels;

			if (SeekSampleNum >= CurrentChunkSampleNum && SeekSampleNum < CurrentChunkSampleNum + NumExpectedTotal)
			{
				CurrentSampleCount = CurrentChunkSampleNum;
				SrcBufferOffset = ChunkPtr - SrcBufferData;
				NumSamplesToSkipInChunk = SeekSampleNum - CurrentChunkSampleNum;
				break;
			}
			CurrentChunkSampleNum += NumExpectedTotal;
			ChunkPtr += ChunkSize + 2;
		}
		// Not found?
		if (ChunkPtr >= EndPtr)
		{
			CurrentSampleCount = SeekSampleNum;
			SrcBufferOffset = SrcBufferDataSize;
		}
	}

	NumRemainingSamplesToSkip = Header.NumSilentSamplesAtBeginning + Header.NumPreSkipSamples + NumSamplesToSkipInChunk;
	PreviousDecodedUnusedSamples.Empty();
}

class OPUSAUDIODECODER_API FOpusAudioDecoderModule : public IModuleInterface
{
public:
	TUniquePtr<IAudioInfoFactory> Factory;

	virtual void StartupModule() override
	{
		Factory = MakeUnique<FSimpleAudioInfoFactory>([] { return new FOpusAudioInfo(); }, Audio::NAME_OPUS);
	}

	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FOpusAudioDecoderModule, OpusAudioDecoder)
