// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpusAudioInfo.h: Unreal audio opus decompression interface object.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "AudioDecompress.h"

struct FOpusDecoderWrapper;
struct FSoundQualityInfo;

/**
* Helper class to parse opus data
*/
class FOpusAudioInfo : public IStreamedCompressedInfo
{
public:
	OPUSAUDIODECODER_API FOpusAudioInfo();
	OPUSAUDIODECODER_API virtual ~FOpusAudioInfo();
	//~ Begin IStreamedCompressedInfo Interface
	bool ParseHeader(const uint8* InSrcBufferData, uint32 InSrcBufferDataSize, FSoundQualityInfo* QualityInfo) override;
	int32 GetFrameSize() override;
	uint32 GetMaxFrameSizeSamples() const override;
	bool CreateDecoder() override;
	FDecodeResult Decode(const uint8* CompressedData, const int32 CompressedDataSize, uint8* OutPCMData, const int32 OutputPCMDataSize) override;
	void PrepareToLoop() override;
	void SeekToTime(const float SeekTime) override;
	void SeekToFrame(const uint32 SeekFrame) override;
	//~ End IStreamedCompressedInfo Interface
	struct FHeader
	{
		char Identifier[8];
		uint8 Version = 0;
		uint8 NumChannels = 0;
		uint32 SampleRate = 0;
		uint32 EncodedSampleRate = 0;
		uint64 ActiveSampleCount = 0;
		uint32 NumEncodedFrames = 0;
		int32 NumPreSkipSamples = 0;
		int32 NumSilentSamplesAtBeginning = 0;
		int32 NumSilentSamplesAtEnd = 0;

		void Reset()
		{
			FMemory::Memzero(Identifier);
			Version = 0;
			NumChannels = 0;
			SampleRate = 0;
			EncodedSampleRate = 0;
			ActiveSampleCount = 0;
			NumEncodedFrames = 0;
			NumPreSkipSamples = 0;
			NumSilentSamplesAtBeginning = 0;
			NumSilentSamplesAtEnd = 0;
		}

		static inline const char OPUS_ID[8] {'U','E','O','P','U','S','\0','\0'};
		static constexpr int32 HeaderSize()
		{
			return
				  sizeof(char)*8	// Identifier
				+ sizeof(uint8)		// Version
				+ sizeof(uint8)		// NumChannels
				+ sizeof(uint32)	// SampleRate
				+ sizeof(uint32)	// EncodedSampleRate
				+ sizeof(uint64)	// ActiveSampleCount
				+ sizeof(uint32)	// NumEncodedFrames
				+ sizeof(int32)		// NumPreSkipSamples
				+ sizeof(int32)		// NumSilentSamplesAtBeginning
				+ sizeof(int32);	// NumSilentSamplesAtEnd
		}
	};
	static OPUSAUDIODECODER_API bool ParseHeader(FHeader& OutHeader, uint32& OutNumRead, const uint8* InSrcBufferData, uint32 InSrcBufferDataSize);
protected:
	/** Wrapper around Opus-specific decoding state and APIs */
	FOpusDecoderWrapper* OpusDecoderWrapper;

	/** The currently active header. */
	FHeader Header;
	
	/** The number of decoded samples that need to be skipped over. */
	int32 NumRemainingSamplesToSkip = 0;

	/** Samples that are left over from the previous Decode() call that did not fit into the receive buffer. */
	TArray<uint8> PreviousDecodedUnusedSamples;
};
